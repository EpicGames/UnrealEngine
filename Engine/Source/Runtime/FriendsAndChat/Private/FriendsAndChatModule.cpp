// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IFriendsAndChatModule.h"
#include "FriendsAndChatStyle.h"


/**
 * Implements the FriendsAndChat module.
 */
class FFriendsAndChatModule
	: public IFriendsAndChatModule
{
public:

	// IModuleInterface interface

	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}

	virtual void ShutdownStyle() override
	{
		FFriendsAndChatModuleStyle::Shutdown();
	}
};


IMPLEMENT_MODULE( FFriendsAndChatModule, FriendsAndChat );
