// Copyright Epic Games, Inc. All Rights Reserved.

#include "FriendsAndChatStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Framework/Application/SlateApplication.h"

const FName FFriendsAndChatStyle::TypeName( TEXT("FFriendsAndChatStyle") );

FFriendsAndChatStyle& FFriendsAndChatStyle::SetSmallFriendsFontStyle(const FFriendsFontStyle& FontStyle)
{
	FriendsSmallFontStyle = FontStyle;
	return *this;
}

FFriendsAndChatStyle& FFriendsAndChatStyle::SetNormalFriendsFontStyle(const FFriendsFontStyle& FontStyle)
{
	FriendsNormalFontStyle = FontStyle;
	return *this;
}

FFriendsAndChatStyle& FFriendsAndChatStyle::SetLargeFriendsFontStyle(const FFriendsFontStyle& FontStyle)
{
	FriendsLargeFontStyle = FontStyle;
	return *this;
}

FFriendsAndChatStyle& FFriendsAndChatStyle::SetChatFontStyle(const FFriendsFontStyle& FontStyle)
{
	ChatFontStyle = FontStyle;
	return *this;
}

FFriendsAndChatStyle& FFriendsAndChatStyle::SetFriendsListStyle(const FFriendsListStyle& InFriendsListStyle)
{
	FriendsListStyle = InFriendsListStyle;
	return *this;
}

FFriendsAndChatStyle& FFriendsAndChatStyle::SetCheckBoxStyle(const FCheckBoxStyle& InCheckBoxStyle)
{
	CheckBoxStyle = InCheckBoxStyle;
	return *this;
}

FFriendsAndChatStyle& FFriendsAndChatStyle::SetRadioBoxStyle(const FCheckBoxStyle& InRadioBoxStyle)
{
	RadioBoxStyle = InRadioBoxStyle;
	return *this;
}

FFriendsAndChatStyle& FFriendsAndChatStyle::SetFriendsChatStyle(const FFriendsChatStyle& InFriendsChatStyle)
{
	FriendsChatStyle = InFriendsChatStyle;
	return *this;
}

FFriendsAndChatStyle& FFriendsAndChatStyle::SetFriendsChatChromeStyle(const FFriendsChatChromeStyle& InFriendsChatChromeStyle)
{
	FriendsChatChromeStyle = InFriendsChatChromeStyle;
	return *this;
}

FFriendsAndChatStyle& FFriendsAndChatStyle::SetFriendsMarkupStyle(const FFriendsMarkupStyle& InFriendsMarkupStyle)
{
	FriendsMarkupStyle = InFriendsMarkupStyle;
	return *this;
}

FFriendsAndChatStyle& FFriendsAndChatStyle::SetScrollbarStyle(const FScrollBarStyle& InScrollBarStyle)
{
	ScrollBarStyle = InScrollBarStyle;
	return *this;
}

FFriendsAndChatStyle& FFriendsAndChatStyle::SetActionButtonStyle(const FButtonStyle& ButtonStyle)
{
	ActionButtonStyle = ButtonStyle;
	return *this;
}

const FFriendsAndChatStyle& FFriendsAndChatStyle::GetDefault()
{
	static FFriendsAndChatStyle Default;
	return Default;
}

/**
	Module style set
*/
TSharedPtr< FSlateStyleSet > FFriendsAndChatModuleStyle::FriendsAndChatModuleStyleInstance = NULL;

void FFriendsAndChatModuleStyle::Initialize(FFriendsAndChatStyle FriendStyle)
{
	if ( !FriendsAndChatModuleStyleInstance.IsValid() )
	{
		FriendsAndChatModuleStyleInstance = Create(FriendStyle);
		FSlateStyleRegistry::RegisterSlateStyle( *FriendsAndChatModuleStyleInstance );
	}
}

void FFriendsAndChatModuleStyle::Shutdown()
{
	if ( FriendsAndChatModuleStyleInstance.IsValid() )
	{
		FSlateStyleRegistry::UnRegisterSlateStyle( *FriendsAndChatModuleStyleInstance );
		ensure( FriendsAndChatModuleStyleInstance.IsUnique() );
		FriendsAndChatModuleStyleInstance.Reset();
	}
}

FName FFriendsAndChatModuleStyle::GetStyleSetName()
{
	static FName StyleSetName("FriendsAndChat");
	return StyleSetName;
}

TSharedRef< FSlateStyleSet > FFriendsAndChatModuleStyle::Create(FFriendsAndChatStyle FriendStyle)
{
	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet("FriendsAndChatStyle"));

	const FButtonStyle UserNameButton = FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetPressed(FSlateNoResource())
		.SetHovered(FSlateNoResource());

	FTextBlockStyle DefaultText;
	FTextBlockStyle FontStyle;
	FHyperlinkStyle HyperlinkStyle;

	// Small
	{
		// DefaultChat
		DefaultText = FTextBlockStyle(FriendStyle.FriendsChatStyle.TextStyle)
			.SetFont(FriendStyle.ChatFontStyle.FriendsFontSmallBold);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(DefaultText)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.DefaultHyperlinkSmall", HyperlinkStyle);
		Style->Set("UserNameTextStyle.DefaultSmall", DefaultText);

		// GlobalChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.GlobalHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.GlobalHyperlinkSmall", HyperlinkStyle);
		Style->Set("UserNameTextStyle.GlobalTextStyleSmall", FontStyle);

		// FounderChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.FounderHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.FounderHyperlinkSmall", HyperlinkStyle);
		Style->Set("UserNameTextStyle.FounderTextStyleSmall", FontStyle);

		// GameChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.GameHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.GameHyperlinkSmall", HyperlinkStyle);
		Style->Set("UserNameTextStyle.GameTextStyleSmall", FontStyle);

		// TeamChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.TeamHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.TeamHyperlinkSmall", HyperlinkStyle);
		Style->Set("UserNameTextStyle.TeamTextStyleSmall", FontStyle);

		// PartyChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.PartyHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.PartyHyperlinkSmall", HyperlinkStyle);
		Style->Set("UserNameTextStyle.PartyTextStyleSmall", FontStyle);

		// WhisperChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.WhisperHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.WhisperlinkSmall", HyperlinkStyle);
		Style->Set("UserNameTextStyle.WhisperTextStyleSmall", FontStyle);

		// MessageBreak
		Style->Set("MessageBreak", FTextBlockStyle(DefaultText)
			.SetFont(FSlateFontInfo(
			FriendStyle.FriendsNormalFontStyle.FriendsFontSmall.FontObject,
			6,
			FriendStyle.FriendsNormalFontStyle.FriendsFontSmall.TypefaceFontName
			)));
	}


	// Normal
	{
		// DefaultChat
		DefaultText = FTextBlockStyle(FriendStyle.FriendsChatStyle.TextStyle)
			.SetFont(FriendStyle.ChatFontStyle.FriendsFontNormalBold);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(DefaultText)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.DefaultHyperlink", HyperlinkStyle);
		Style->Set("UserNameTextStyle.Default", DefaultText);

		// GlobalChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.GlobalHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.GlobalHyperlink", HyperlinkStyle);
		Style->Set("UserNameTextStyle.GlobalTextStyle", FontStyle);

		// FounderChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.FounderHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.FounderHyperlink", HyperlinkStyle);
		Style->Set("UserNameTextStyle.FounderTextStyle", FontStyle);

		// GameChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.GameHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.GameHyperlink", HyperlinkStyle);
		Style->Set("UserNameTextStyle.GameTextStyle", FontStyle);

		// TeamChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.TeamHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.TeamHyperlink", HyperlinkStyle);
		Style->Set("UserNameTextStyle.TeamTextStyle", FontStyle);

		// PartyChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.PartyHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.PartyHyperlink", HyperlinkStyle);
		Style->Set("UserNameTextStyle.PartyTextStyle", FontStyle);

		// WhisperChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.WhisperHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.Whisperlink", HyperlinkStyle);
		Style->Set("UserNameTextStyle.WhisperTextStyle", FontStyle);
	}

	// Large
	{
		// DefaultChat
		DefaultText = FTextBlockStyle(FriendStyle.FriendsChatStyle.TextStyle)
			.SetFont(FriendStyle.ChatFontStyle.FriendsFontLargeBold);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(DefaultText)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.DefaultHyperlinkLarge", HyperlinkStyle);
		Style->Set("UserNameTextStyle.DefaultLarge", DefaultText);

		// GlobalChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.GlobalHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.GlobalHyperlinkLarge", HyperlinkStyle);
		Style->Set("UserNameTextStyle.GlobalTextStyleLarge", FontStyle);

		// FounderChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.FounderHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.FounderHyperlinkLarge", HyperlinkStyle);
		Style->Set("UserNameTextStyle.FounderTextStyleLarge", FontStyle);

		// GameChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.GameHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.GameHyperlinkLarge", HyperlinkStyle);
		Style->Set("UserNameTextStyle.GameTextStyleLarge", FontStyle);

		// TeamChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.TeamHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.TeamHyperlinkLarge", HyperlinkStyle);
		Style->Set("UserNameTextStyle.TeamTextStyleLarge", FontStyle);

		// PartyChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.PartyHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.PartyHyperlinkLarge", HyperlinkStyle);
		Style->Set("UserNameTextStyle.PartyTextStyleLarge", FontStyle);

		// WhisperChat
		FontStyle = FTextBlockStyle(DefaultText)
			.SetColorAndOpacity(FriendStyle.FriendsChatStyle.WhisperHyperlinkChatColor);

		HyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(UserNameButton)
			.SetTextStyle(FontStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("UserNameTextStyle.WhisperlinkLarge", HyperlinkStyle);
		Style->Set("UserNameTextStyle.WhisperTextStyleLarge", FontStyle);
	}

	return Style;
}

void FFriendsAndChatModuleStyle::ReloadTextures()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

const ISlateStyle& FFriendsAndChatModuleStyle::Get()
{
	return *FriendsAndChatModuleStyleInstance;
}
