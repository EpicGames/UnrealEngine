// Copyright Epic Games, Inc. All Rights Reserved.

#include "FriendsChatChromeStyle.h"

const FName FFriendsChatChromeStyle::TypeName( TEXT("FFriendsChatChromeStyle") );

const FFriendsChatChromeStyle& FFriendsChatChromeStyle::GetDefault()
{
	static FFriendsChatChromeStyle Default;
	return Default;
}


FFriendsChatChromeStyle& FFriendsChatChromeStyle::SetChatBackgroundBrush(const FSlateBrush& Value)
{
	ChatBackgroundBrush = Value;
	return *this;
}

FFriendsChatChromeStyle& FFriendsChatChromeStyle::SetChatEntryBackgroundBrush(const FSlateBrush& Value)
{
	ChatEntryBackgroundBrush = Value;
	return *this;
}

FFriendsChatChromeStyle& FFriendsChatChromeStyle::SetChannelBackgroundBrush(const FSlateBrush& Value)
{
	ChannelBackgroundBrush = Value;
	return *this;
}

FFriendsChatChromeStyle& FFriendsChatChromeStyle::SetChatEntryBackgroundColor(const FLinearColor& Value)
{
	ChatEntryBackgroundColor = Value;
	return *this;
}

FFriendsChatChromeStyle& FFriendsChatChromeStyle::SetTabBackgroundBrush(const FSlateBrush& Value)
{
	TabBackgroundBrush = Value;
	return *this;
}

FFriendsChatChromeStyle& FFriendsChatChromeStyle::SetNoneActiveTabColor(const FLinearColor& Value)
{
	NoneActiveTabColor = Value;
	return *this;
}

FFriendsChatChromeStyle& FFriendsChatChromeStyle::SetTabFontColor(const FLinearColor& Value)
{
	TabFontColor = Value;
	return *this;
}

FFriendsChatChromeStyle& FFriendsChatChromeStyle::SetTabFontColorInverted(const FLinearColor& Value)
{
	TabFontColorInverted = Value;
	return *this;
}


FFriendsChatChromeStyle& FFriendsChatChromeStyle::SetTabWidth(float Value)
{
	TabWidth = Value;
	return *this;
}

FFriendsChatChromeStyle& FFriendsChatChromeStyle::SetTabPadding(const FMargin& Value)
{
	TabPadding = Value;
	return *this;
}

FFriendsChatChromeStyle& FFriendsChatChromeStyle::SetChatWindowPadding(const FMargin& Value)
{
	ChatWindowPadding = Value;
	return *this;
}


FFriendsChatChromeStyle& FFriendsChatChromeStyle::SetChatWindowToEntryMargin(const FMargin& Value)
{
	ChatWindowToEntryMargin = Value;
	return *this;
}

FFriendsChatChromeStyle& FFriendsChatChromeStyle::SetChatChannelPadding(const FMargin& Value)
{
	ChatChannelPadding = Value;
	return *this;
}


FFriendsChatChromeStyle& FFriendsChatChromeStyle::SetChatEntryPadding(const FMargin& Value)
{
	ChatEntryPadding = Value;
	return *this;
}

FFriendsChatChromeStyle& FFriendsChatChromeStyle::SetChatBackgroundColor(const FLinearColor& Value)
{
	ChatBackgroundColor = Value;
	return *this;
}

FFriendsChatChromeStyle& FFriendsChatChromeStyle::SetChatMenuBackgroundBrush(const FSlateBrush& Value)
{
	ChatMenuBackgroundBrush = Value;
	return *this;
}
