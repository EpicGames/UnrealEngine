// Copyright Epic Games, Inc. All Rights Reserved.

#include "FriendsChatStyle.h"

const FName FFriendsChatStyle::TypeName( TEXT("FFriendsChatStyle") );

const FFriendsChatStyle& FFriendsChatStyle::GetDefault()
{
	static FFriendsChatStyle Default;
	return Default;
}

FFriendsChatStyle& FFriendsChatStyle::SetTextStyle(const FTextBlockStyle& InTextStle)
{
	TextStyle = InTextStle;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetTimeStampTextStyle(const FTextBlockStyle& InTextStle)
{
	TimeStampTextStyle = InTextStle;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetTimestampOpacity(float Value)
{
	TimeStampOpacity = Value;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetDefaultChatColor(const FLinearColor& InFontColor)
{
	DefaultChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetGlobalChatColor(const FLinearColor& InFontColor)
{
	GlobalChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetFounderChatColor(const FLinearColor& InFontColor)
{
	FounderChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetWhisplerChatColor(const FLinearColor& InFontColor)
{
	WhisperChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetGameChatColor(const FLinearColor& InFontColor)
{
	GameChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetTeamChatColor(const FLinearColor& InFontColor)
{
	TeamChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetPartyChatColor(const FLinearColor& InFontColor)
{
	PartyChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetAdminChatColor(const FLinearColor& InFontColor)
{
	AdminChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetGameAdminChatColor(const FLinearColor& InFontColor)
{
	GameAdminChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetAddedItemChatColor(const FLinearColor& InFontColor)
{
	AddedItemChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetCompletedItemChatColor(const FLinearColor& InFontColor)
{
	CompletedItemChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetDiscardedItemChatColor(const FLinearColor& InFontColor)
{
	DiscardedItemChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetWhisplerHyperlinkChatColor(const FLinearColor& InFontColor)
{
	WhisperHyperlinkChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetGlobalHyperlinkChatColor(const FLinearColor& InFontColor)
{
	GlobalHyperlinkChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetFounderHyperlinkChatColor(const FLinearColor& InFontColor)
{
	FounderHyperlinkChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetGameHyperlinkChatColor(const FLinearColor& InFontColor)
{
	GameHyperlinkChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetTeamHyperlinkChatColor(const FLinearColor& InFontColor)
{
	TeamHyperlinkChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetPartyHyperlinkChatColor(const FLinearColor& InFontColor)
{
	PartyHyperlinkChatColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetEnemyColor(const FLinearColor& InFontColor)
{
	EnemyColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetFriendlyColor(const FLinearColor& InFontColor)
{
	FriendlyColor = InFontColor;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetChatEntryTextStyle(const FEditableTextBoxStyle& InEditableTextStyle)
{
	ChatEntryTextStyle = InEditableTextStyle;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetChatDisplayTextStyle(const FEditableTextBoxStyle& InEditableTextStyle)
{
	ChatDisplayTextStyle = InEditableTextStyle;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetScrollBorderStyle(const FScrollBoxStyle& InScrollBorderStyle)
{
	ScrollBorderStyle = InScrollBorderStyle;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetMessageNotificationBrush(const FSlateBrush& Brush)
{
	MessageNotificationBrush = Brush;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetChatChannelPadding(const FMargin& Value)
{
	ChatEntryPadding = Value;
	return *this;
}

FFriendsChatStyle& FFriendsChatStyle::SetChatEntryHeight(float Value)
{
	ChatEntryHeight = Value;
	return *this;
}
