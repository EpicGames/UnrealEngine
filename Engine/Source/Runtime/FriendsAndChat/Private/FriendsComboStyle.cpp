// Copyright Epic Games, Inc. All Rights Reserved.

#include "FriendsComboStyle.h"

const FName FFriendsComboStyle::TypeName( TEXT("FFriendsComboStyle") );

const FFriendsComboStyle& FFriendsComboStyle::GetDefault()
{
	static FFriendsComboStyle Default;
	return Default;
}

FFriendsComboStyle& FFriendsComboStyle::SetComboItemButtonStyle(const FButtonStyle& ButtonStyle)
{
	ComboItemButtonStyle = ButtonStyle;
	return *this;
}

FFriendsComboStyle& FFriendsComboStyle::SetComboItemTextColorNormal(const FLinearColor& InColor)
{
	ComboItemTextColorNormal = InColor;
	return *this;
}

FFriendsComboStyle& FFriendsComboStyle::SetComboItemTextColorInverted(const FLinearColor& InColor)
{
	ComboItemTextColorInverted = InColor;
	return *this;
}

FFriendsComboStyle& FFriendsComboStyle::SetComboItemTextStyle(const FTextBlockStyle& InTextStyle)
{
	ComboItemTextStyle = InTextStyle;
	return *this;
}

FFriendsComboStyle& FFriendsComboStyle::SetFriendsComboTextStyle(const FTextBlockStyle& InTextStyle)
{
	FriendsComboTextStyle = InTextStyle;
	return *this;
}

FFriendsComboStyle& FFriendsComboStyle::SetStatusButtonSize(const FVector2D& InStatusButtonSize)
{
	StatusButtonSize = InStatusButtonSize;
	return *this;
}

FFriendsComboStyle& FFriendsComboStyle::SetActionComboButtonSize(const FVector2D& InActionComboButtonSize)
{
	ActionComboButtonSize = InActionComboButtonSize;
	return *this;
}

FFriendsComboStyle& FFriendsComboStyle::SetActionComboButtonStyle(const FComboButtonStyle& InActionComboButtonStyle)
{
	ActionComboButtonStyle = InActionComboButtonStyle;
	return *this;
}

FFriendsComboStyle& FFriendsComboStyle::SetActionComboButtonTextStyle(const FTextBlockStyle& InActionComboButtonTextStyle)
{
	ActionComboButtonTextStyle = InActionComboButtonTextStyle;
	return *this;
}

FFriendsComboStyle& FFriendsComboStyle::SetComboMenuPadding(const FMargin& InPadding)
{
	ComboMenuPadding = InPadding;
	return *this;
}

FFriendsComboStyle& FFriendsComboStyle::SetComboItemPadding(const FMargin& InPadding)
{
	ComboItemPadding = InPadding;
	return *this;
}

FFriendsComboStyle& FFriendsComboStyle::SetComboItemContentPadding(const FMargin& InPadding)
{
	ComboItemContentPadding = InPadding;
	return *this;
}

/** Friends List Combo Button menu background image (left) */
FFriendsComboStyle& FFriendsComboStyle::SetFriendComboBackgroundLeftBrush(const FSlateBrush& BrushStyle)
{
	FriendComboBackgroundLeftBrush = BrushStyle;
	return *this;
}

/** Friends List Combo Button menu background image (right) */
FFriendsComboStyle& FFriendsComboStyle::SetFriendComboBackgroundRightBrush(const FSlateBrush& BrushStyle)
{
	FriendComboBackgroundRightBrush = BrushStyle;
	return *this;
}

/** Friends List Combo Button menu background image (left-flipped) - for MenuPlacement_ComboBoxRight menus */
FFriendsComboStyle& FFriendsComboStyle::SetFriendComboBackgroundLeftFlippedBrush(const FSlateBrush& BrushStyle)
{
	FriendComboBackgroundLeftFlippedBrush = BrushStyle;
	return *this;
}

/** Friends List Combo Button menu background image (right-flipped) - for MenuPlacement_ComboBoxRight menus */
FFriendsComboStyle& FFriendsComboStyle::SetFriendComboBackgroundRightFlippedBrush(const FSlateBrush& BrushStyle)
{
	FriendComboBackgroundRightFlippedBrush = BrushStyle;
	return *this;
}

FFriendsComboStyle& FFriendsComboStyle::SetFriendsListComboButtonStyle(const FComboButtonStyle& ButtonStyle)
{
	FriendListComboButtonStyle = ButtonStyle;
	return *this;
}

