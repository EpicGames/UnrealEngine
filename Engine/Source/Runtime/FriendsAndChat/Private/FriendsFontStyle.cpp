// Copyright Epic Games, Inc. All Rights Reserved.

#include "FriendsFontStyle.h"

const FName FFriendsFontStyle::TypeName( TEXT("FFriendsFontStyle") );

const FFriendsFontStyle& FFriendsFontStyle::GetDefault()
{
	static FFriendsFontStyle Default;
	return Default;
}

FFriendsFontStyle& FFriendsFontStyle::SetFontSmall(const FSlateFontInfo& FontStyle)
{
	FriendsFontSmall = FontStyle;
	return *this;
}

FFriendsFontStyle& FFriendsFontStyle::SetFontSmallBold(const FSlateFontInfo& FontStyle)
{
	FriendsFontSmallBold = FontStyle;
	return *this;
}

FFriendsFontStyle& FFriendsFontStyle::SetFontNormal(const FSlateFontInfo& FontStyle)
{
	FriendsFontNormal = FontStyle;
	return *this;
}

FFriendsFontStyle& FFriendsFontStyle::SetFontNormalBold(const FSlateFontInfo& FontStyle)
{
	FriendsFontNormalBold = FontStyle;
	return *this;
}

FFriendsFontStyle& FFriendsFontStyle::SetFontLarge(const FSlateFontInfo& FontStyle)
{
	FriendsFontLarge = FontStyle;
	return *this;
}

FFriendsFontStyle& FFriendsFontStyle::SetFontLargeBold(const FSlateFontInfo& FontStyle)
{
	FriendsFontLargeBold = FontStyle;
	return *this;
}

FFriendsFontStyle& FFriendsFontStyle::SetChatFont(const FSlateFontInfo& FontStyle)
{
	FriendsChatFont = FontStyle;
	return *this;
}

FFriendsFontStyle& FFriendsFontStyle::SetDefaultFontColor(const FLinearColor& FontColor)
{
	DefaultFontColor = FontColor;
	return *this;
}

FFriendsFontStyle& FFriendsFontStyle::SetInvertedFontColor(const FLinearColor& FontColor)
{
	InvertedFontColor = FontColor;
	return *this;
}

FFriendsFontStyle& FFriendsFontStyle::SetDefaultDullFontColor(const FLinearColor& FontColor)
{
	DefaultDullFontColor = FontColor;
	return *this;
}
