// Copyright Epic Games, Inc. All Rights Reserved.

#include "FriendsListStyle.h"

const FName FFriendsListStyle::TypeName( TEXT("FFriendsListStyle") );

const FFriendsListStyle& FFriendsListStyle::GetDefault()
{
	static FFriendsListStyle Default;
	return Default;
}

FFriendsListStyle& FFriendsListStyle::SetGlobalChatButtonStyle(const FButtonStyle& ButtonStyle)
{
	GlobalChatButtonStyle = ButtonStyle;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetGlobalChatIcon(const FSlateBrush& BrushStyle)
{
	GlobalChatIcon = BrushStyle;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetFriendItemButtonStyle(const FButtonStyle& ButtonStyle)
{
	FriendItemButtonStyle = ButtonStyle;
	return *this;
}


FFriendsListStyle& FFriendsListStyle::SetConfirmButtonStyle(const FButtonStyle& ButtonStyle)
{
	ConfirmButtonStyle = ButtonStyle;
	return *this;
}


FFriendsListStyle& FFriendsListStyle::SetCancelButtonStyle(const FButtonStyle& ButtonStyle)
{
	CancelButtonStyle = ButtonStyle;
	return *this;
}


FFriendsListStyle& FFriendsListStyle::SetButtonContentColor(const FSlateColor& InColor)
{
	ButtonContentColor = InColor;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetButtonHoverContentColor(const FSlateColor& InColor)
{
	ButtonHoverContentColor = InColor;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetActionMenuArrowBrush(const FSlateBrush& BrushStyle)
{
	ActionMenuArrowBrush = BrushStyle;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetToolTipArrowBrush(const FSlateBrush& BrushStyle)
{
	ToolTipArrowBrush = BrushStyle;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetBackButtonStyle(const FButtonStyle& ButtonStyle)
{
	BackButtonStyle = ButtonStyle;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetHeaderButtonStyle(const FButtonStyle& ButtonStyle)
{
	HeaderButtonStyle = ButtonStyle;
	return *this;
}

/** Friends List Action Button style */
FFriendsListStyle& FFriendsListStyle::SetFriendListActionButtonStyle(const FButtonStyle& ButtonStyle)
{
	FriendListActionButtonStyle = ButtonStyle;
	return *this;
}

/** Optional content for the Add Friend button */
FFriendsListStyle& FFriendsListStyle::SetAddFriendButtonContentBrush(const FSlateBrush& BrushStyle)
{
	AddFriendButtonContentBrush = BrushStyle;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetStatusIconBrush(const FSlateBrush& BrushStyle)
{
	StatusIconBrush = BrushStyle;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetPCIconBrush(const FSlateBrush& BrushStyle)
{
	PCIconBrush = BrushStyle;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetConsoleIconBrush(const FSlateBrush& BrushStyle)
{
	ConsoleIconBrush = BrushStyle;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetEpicIconBrush(const FSlateBrush& BrushStyle)
{
	EpicIconBrush = BrushStyle;
	return *this;
}


/** Friend Image brush style */
FFriendsListStyle& FFriendsListStyle::SetFriendImageBrush(const FSlateBrush& BrushStyle)
{
	FriendImageBrush = BrushStyle;
	return *this;
}

/** Offline brush style */
FFriendsListStyle& FFriendsListStyle::SetOfflineBrush(const FSlateBrush& BrushStyle)
{
	OfflineBrush = BrushStyle;
	return *this;
}

/** Online brush style */
FFriendsListStyle& FFriendsListStyle::SetOnlineBrush(const FSlateBrush& BrushStyle)
{
	OnlineBrush = BrushStyle;
	return *this;
}

/** Away brush style */
FFriendsListStyle& FFriendsListStyle::SetAwayBrush(const FSlateBrush& BrushStyle)
{
	AwayBrush = BrushStyle;
	return *this;
}

/** Away brush style */
FFriendsListStyle& FFriendsListStyle::SetSpectateBrush(const FSlateBrush& BrushStyle)
{
	SpectateBrush = BrushStyle;
	return *this;
}

/** Friends window background */
FFriendsListStyle& FFriendsListStyle::SetFriendContainerBackground(const FSlateBrush& BrushStyle)
{
	FriendsContainerBackground = BrushStyle;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetFriendsListBackground(const FSlateBrush& BrushStyle)
{
	FriendsListBackground = BrushStyle;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetAddFriendEditableTextStyle(const FEditableTextBoxStyle& TextStyle)
{
	AddFriendEditableTextStyle = TextStyle;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetUserPresenceImageSize(const FVector2D& Size)
{
	UserPresenceImageSize = Size;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetBackBrush(const FSlateBrush& Brush)
{
	BackBrush = Brush;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetSelectedOptionBrush(const FSlateBrush& Brush)
{
	SelectedOptionBrush = Brush;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetSettingsBrush(const FSlateBrush& Brush)
{
	SettingsBrush = Brush;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetSeperatorBrush(const FSlateBrush& Brush)
{
	SeperatorBrush = Brush;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetPresenceSeperatorBrush(const FSlateBrush& Brush)
{
	PresenceSeperatorBrush = Brush;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetFontSizeBrush(const FSlateBrush& Brush)
{
	FontSizeBrush = Brush;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetSearchBrush(const FSlateBrush& Brush)
{
	SearchBrush = Brush;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetClanDetailsBrush(const FSlateBrush& Brush)
{
	ClanDetailsBrush = Brush;
	return *this;
}

FFriendsListStyle& FFriendsListStyle::SetClanMembersBrush(const FSlateBrush& Brush)
{
	ClanMembersBrush = Brush;
	return *this;
}
