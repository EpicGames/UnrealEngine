// Copyright Epic Games, Inc. All Rights Reserved.

#include "FriendsMarkupStyle.h"

const FName FFriendsMarkupStyle::TypeName( TEXT("FFriendsMarkupStyle") );

const FFriendsMarkupStyle& FFriendsMarkupStyle::GetDefault()
{
	static FFriendsMarkupStyle Default;
	return Default;
}

FFriendsMarkupStyle& FFriendsMarkupStyle::SetMarkupButtonStyle(const FButtonStyle& ButtonStyle)
{
	MarkupButtonStyle = ButtonStyle;
	return *this;
}

FFriendsMarkupStyle& FFriendsMarkupStyle::SetMarkupTextStyle(const FTextBlockStyle& InTextStyle)
{
	MarkupTextStyle = InTextStyle;
	return *this;
}

FFriendsMarkupStyle& FFriendsMarkupStyle::SetMarkupBackgroundBrush(const FSlateBrush& Value)
{
	MarkupBackground = Value;
	return *this;
}

FFriendsMarkupStyle& FFriendsMarkupStyle::SetButtonColor(const FSlateColor& Value)
{
	ButtonColor = Value;
	return *this;
}

FFriendsMarkupStyle& FFriendsMarkupStyle::SetButtonHoverColor(const FSlateColor& Value)
{
	ButtonHoverColor = Value;
	return *this;
}

FFriendsMarkupStyle& FFriendsMarkupStyle::SetTipColor(const FSlateColor& Value)
{
	TipColor = Value;
	return *this;
}

FFriendsMarkupStyle& FFriendsMarkupStyle::SetSeperatorBrush(const FSlateBrush& Value)
{
	SeperatorBrush = Value;
	return *this;
}
