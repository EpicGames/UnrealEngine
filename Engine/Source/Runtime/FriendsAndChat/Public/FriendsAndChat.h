// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

/* Dependencies
 *****************************************************************************/

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "SlateBasics.h"
#include "Styling/SlateStyle.h"

/* Interfaces
 *****************************************************************************/
#include "Interfaces/IFriendsAndChatModule.h"
#include "FriendsFontStyle.h"
#include "FriendsComboStyle.h"
#include "FriendsListStyle.h"
#include "FriendsChatStyle.h"
#include "FriendsChatChromeStyle.h"
#include "FriendsAndChatStyle.h"
