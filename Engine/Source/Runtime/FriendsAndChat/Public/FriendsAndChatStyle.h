// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateTypes.h"
#include "FriendsFontStyle.h"
#include "FriendsListStyle.h"
#include "FriendsChatStyle.h"
#include "FriendsChatChromeStyle.h"
#include "FriendsMarkupStyle.h"

#include "FriendsAndChatStyle.generated.h"

class ISlateStyle;
struct FSlateBrush;

/**
 * Interface for the services manager.
 */
USTRUCT()
struct FRIENDSANDCHAT_API FFriendsAndChatStyle
	: public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	// Default Constructor
	FFriendsAndChatStyle() { }

	// Default Destructor
	virtual ~FFriendsAndChatStyle() { }

	/**
	 * Override widget style function.
	 */
	virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override { }

	// Holds the widget type name
	static const FName TypeName;

	/**
	 * Get the type name.
	 * @return the type name
	 */
	virtual const FName GetTypeName() const override { return TypeName; };

	/**
	 * Get the default style.
	 * @return the default style
	 */
	static const FFriendsAndChatStyle& GetDefault();

	// Common Style

	UPROPERTY(EditAnywhere, Category = Appearance)
	FScrollBarStyle ScrollBarStyle;
	FFriendsAndChatStyle& SetScrollbarStyle(const FScrollBarStyle& InScrollBarStyle);

	/** SFriendActions Action Button style */
	UPROPERTY( EditAnywhere, Category = Appearance )
	FButtonStyle ActionButtonStyle;
	FFriendsAndChatStyle& SetActionButtonStyle(const FButtonStyle& ButtonStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FFriendsFontStyle FriendsSmallFontStyle;
	FFriendsAndChatStyle& SetSmallFriendsFontStyle(const FFriendsFontStyle& FontStyle);

	UPROPERTY( EditAnywhere, Category = Appearance )
	FFriendsFontStyle FriendsNormalFontStyle;
	FFriendsAndChatStyle& SetNormalFriendsFontStyle(const FFriendsFontStyle& FontStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FFriendsFontStyle FriendsLargeFontStyle;
	FFriendsAndChatStyle& SetLargeFriendsFontStyle(const FFriendsFontStyle& FontStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FFriendsFontStyle ChatFontStyle;
	FFriendsAndChatStyle& SetChatFontStyle(const FFriendsFontStyle& FontStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FCheckBoxStyle CheckBoxStyle;
	FFriendsAndChatStyle& SetCheckBoxStyle(const FCheckBoxStyle& InCheckBoxStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FCheckBoxStyle RadioBoxStyle;
	FFriendsAndChatStyle& SetRadioBoxStyle(const FCheckBoxStyle& InRadioBoxStyle);


	UPROPERTY( EditAnywhere, Category = Appearance )
	FFriendsListStyle FriendsListStyle;
	FFriendsAndChatStyle& SetFriendsListStyle(const FFriendsListStyle& InFriendsListStyle);

	UPROPERTY( EditAnywhere, Category = Appearance )
	FFriendsChatStyle FriendsChatStyle;
	FFriendsAndChatStyle& SetFriendsChatStyle(const FFriendsChatStyle& InFriendsChatStyle);

	UPROPERTY( EditAnywhere, Category = Appearance )
	FFriendsChatChromeStyle FriendsChatChromeStyle;
	FFriendsAndChatStyle& SetFriendsChatChromeStyle(const FFriendsChatChromeStyle& InFriendsChatChromeStyle);

	UPROPERTY( EditAnywhere, Category = Appearance )
	FFriendsMarkupStyle FriendsMarkupStyle;
	FFriendsAndChatStyle& SetFriendsMarkupStyle(const FFriendsMarkupStyle& InFriendsMarkupStyle);
};

/** Manages the style which provides resources for the rich text widget. */
class FRIENDSANDCHAT_API FFriendsAndChatModuleStyle
{
public:

	static void Initialize(FFriendsAndChatStyle FriendStyle);

	static void Shutdown();

	/** reloads textures used by slate renderer */
	static void ReloadTextures();

	/** @return The Slate style set for the Friends and chat module */
	static const ISlateStyle& Get();

	static FName GetStyleSetName();

private:

	static TSharedRef< class FSlateStyleSet > Create(FFriendsAndChatStyle FriendStyle);

private:

	static TSharedPtr< class FSlateStyleSet > FriendsAndChatModuleStyleInstance;
};
