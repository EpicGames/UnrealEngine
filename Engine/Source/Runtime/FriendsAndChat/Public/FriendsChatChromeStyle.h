// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NameTypes.h"
#include "Math/Color.h"
#include "UObject/ObjectMacros.h"
#include "Layout/Margin.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateWidgetStyle.h"
#include "FriendsChatChromeStyle.generated.h"

/**
 * Interface for the services manager.
 */
USTRUCT()
struct FRIENDSANDCHAT_API FFriendsChatChromeStyle
	: public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	// Default Constructor
	FFriendsChatChromeStyle() { }

	// Default Destructor
	virtual ~FFriendsChatChromeStyle() { }

	/**
	 * Override widget style function.
	 */
	virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override { }

	// Holds the widget type name
	static const FName TypeName;

	/**
	 * Get the type name.
	 * @return the type name
	 */
	virtual const FName GetTypeName() const override { return TypeName; };

	/**
	 * Get the default style.
	 * @return the default style
	 */
	static const FFriendsChatChromeStyle& GetDefault();

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush ChatBackgroundBrush;
	FFriendsChatChromeStyle& SetChatBackgroundBrush(const FSlateBrush& InChatBackgroundBrush);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush ChatEntryBackgroundBrush;
	FFriendsChatChromeStyle& SetChatEntryBackgroundBrush(const FSlateBrush& InChatBackgroundBrush);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush ChannelBackgroundBrush;
	FFriendsChatChromeStyle& SetChannelBackgroundBrush(const FSlateBrush& Value);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor ChatEntryBackgroundColor;
	FFriendsChatChromeStyle& SetChatEntryBackgroundColor(const FLinearColor& Value);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush TabBackgroundBrush;
	FFriendsChatChromeStyle& SetTabBackgroundBrush(const FSlateBrush& InTabBackgroundBrush);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor NoneActiveTabColor;
	FFriendsChatChromeStyle& SetNoneActiveTabColor(const FLinearColor& Value);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor TabFontColor;
	FFriendsChatChromeStyle& SetTabFontColor(const FLinearColor& Value);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor TabFontColorInverted;
	FFriendsChatChromeStyle& SetTabFontColorInverted(const FLinearColor& Value);

	UPROPERTY(EditAnywhere, Category = Appearance)
	float TabWidth;
	FFriendsChatChromeStyle& SetTabWidth(float Value);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin TabPadding;
	FFriendsChatChromeStyle& SetTabPadding(const FMargin& Value);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin ChatWindowPadding;
	FFriendsChatChromeStyle& SetChatWindowPadding(const FMargin& Value);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin ChatWindowToEntryMargin;
	FFriendsChatChromeStyle& SetChatWindowToEntryMargin(const FMargin& Value);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin ChatChannelPadding;
	FFriendsChatChromeStyle& SetChatChannelPadding(const FMargin& Value);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin ChatEntryPadding;
	FFriendsChatChromeStyle& SetChatEntryPadding(const FMargin& Value);

	UPROPERTY(EditAnywhere, Category=Appearance)
	FLinearColor ChatBackgroundColor;
	FFriendsChatChromeStyle& SetChatBackgroundColor(const FLinearColor& Value);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush ChatMenuBackgroundBrush;
	FFriendsChatChromeStyle& SetChatMenuBackgroundBrush(const FSlateBrush& InChatMenuBackgroundBrush);
};

