// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NameTypes.h"
#include "Math/Color.h"
#include "UObject/ObjectMacros.h"
#include "Layout/Margin.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/SlateTypes.h"
#include "FriendsChatStyle.generated.h"

// Forward declarations
namespace EChatMessageType
{
	enum Type : uint16;
}

/**
 * Interface for the services manager.
 */
USTRUCT()
struct FRIENDSANDCHAT_API FFriendsChatStyle
	: public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	// Default Constructor
	FFriendsChatStyle()
		: TimeStampOpacity(0.5f)
		, ChatEntryPadding(0)
		, ChatEntryHeight(55)
		, FriendActionPadding(20, 15)
		, FriendActionHeaderPadding(20)
		, FriendActionStatusMargin(20, 0, 0, 0)
	{

	}

	// Default Destructor
	virtual ~FFriendsChatStyle() { }

	/**
	 * Override widget style function.
	 */
	virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override { }

	// Holds the widget type name
	static const FName TypeName;

	/**
	 * Get the type name.
	 * @return the type name
	 */
	virtual const FName GetTypeName() const override { return TypeName; };

	/**
	 * Get the default style.
	 * @return the default style
	 */
	static const FFriendsChatStyle& GetDefault();

	/** Text Style */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FTextBlockStyle TextStyle;
	FFriendsChatStyle& SetTextStyle(const FTextBlockStyle& InTextStyle);

	/** Time Stamp Text Style */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FTextBlockStyle TimeStampTextStyle;
	FFriendsChatStyle& SetTimeStampTextStyle(const FTextBlockStyle& InTextStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	float TimeStampOpacity;
	FFriendsChatStyle& SetTimestampOpacity(float InOpacity);

	UPROPERTY(EditAnywhere, Category=Appearance)
	FLinearColor DefaultChatColor;
	FFriendsChatStyle& SetDefaultChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category=Appearance)
	FLinearColor WhisperChatColor;
	FFriendsChatStyle& SetWhisplerChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor GlobalChatColor;
	FFriendsChatStyle& SetGlobalChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor FounderChatColor;
	FFriendsChatStyle& SetFounderChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category=Appearance)
	FLinearColor GameChatColor;
	FFriendsChatStyle& SetGameChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category=Appearance)
	FLinearColor TeamChatColor;
	FFriendsChatStyle& SetTeamChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor PartyChatColor;
	FFriendsChatStyle& SetPartyChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category=Appearance)
	FLinearColor AdminChatColor;
	FFriendsChatStyle& SetAdminChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category=Appearance)
	FLinearColor GameAdminChatColor;
	FFriendsChatStyle& SetGameAdminChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor AddedItemChatColor;
	FFriendsChatStyle& SetAddedItemChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor CompletedItemChatColor;
	FFriendsChatStyle& SetCompletedItemChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor DiscardedItemChatColor;
	FFriendsChatStyle& SetDiscardedItemChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category=Appearance)
	FLinearColor WhisperHyperlinkChatColor;
	FFriendsChatStyle& SetWhisplerHyperlinkChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor GlobalHyperlinkChatColor;
	FFriendsChatStyle& SetGlobalHyperlinkChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor FounderHyperlinkChatColor;
	FFriendsChatStyle& SetFounderHyperlinkChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category=Appearance)
	FLinearColor GameHyperlinkChatColor;
	FFriendsChatStyle& SetGameHyperlinkChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category=Appearance)
	FLinearColor TeamHyperlinkChatColor;
	FFriendsChatStyle& SetTeamHyperlinkChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor PartyHyperlinkChatColor;
	FFriendsChatStyle& SetPartyHyperlinkChatColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor EnemyColor;
	FFriendsChatStyle& SetEnemyColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor FriendlyColor;
	FFriendsChatStyle& SetFriendlyColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FEditableTextBoxStyle ChatEntryTextStyle;
	FFriendsChatStyle& SetChatEntryTextStyle(const FEditableTextBoxStyle& InEditableTextStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FEditableTextBoxStyle ChatDisplayTextStyle;
	FFriendsChatStyle& SetChatDisplayTextStyle(const FEditableTextBoxStyle& InEditableTextStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FScrollBoxStyle ScrollBorderStyle;
	FFriendsChatStyle& SetScrollBorderStyle(const FScrollBoxStyle& InScrollBorderStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush MessageNotificationBrush;
	FFriendsChatStyle& SetMessageNotificationBrush(const FSlateBrush& Brush);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin ChatEntryPadding;
	FFriendsChatStyle& SetChatChannelPadding(const FMargin& Value);

	UPROPERTY(EditAnywhere, Category = Appearance)
	float ChatEntryHeight;
	FFriendsChatStyle& SetChatEntryHeight(float Value);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin FriendActionPadding;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin FriendActionHeaderPadding;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin FriendActionStatusMargin;
};

