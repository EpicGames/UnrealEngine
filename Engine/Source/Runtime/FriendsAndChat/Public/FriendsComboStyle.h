// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NameTypes.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "UObject/ObjectMacros.h"
#include "Layout/Margin.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/SlateTypes.h"
#include "FriendsComboStyle.generated.h"

/**
 * Interface for the services manager.
 */
USTRUCT()
struct FRIENDSANDCHAT_API FFriendsComboStyle
	: public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	// Default Constructor
	FFriendsComboStyle() { }

	// Default Destructor
	virtual ~FFriendsComboStyle() { }

	/**
	 * Override widget style function.
	 */
	virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override { }

	// Holds the widget type name
	static const FName TypeName;

	/**
	 * Get the type name.
	 * @return the type name
	 */
	virtual const FName GetTypeName() const override { return TypeName; };

	/**
	 * Get the default style.
	 * @return the default style
	 */
	static const FFriendsComboStyle& GetDefault();

	UPROPERTY(EditAnywhere, Category = Appearance)
	FButtonStyle ComboItemButtonStyle;
	FFriendsComboStyle& SetComboItemButtonStyle(const FButtonStyle& ButtonStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor ComboItemTextColorNormal;
	FFriendsComboStyle& SetComboItemTextColorNormal(const FLinearColor& InColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor ComboItemTextColorInverted;
	FFriendsComboStyle& SetComboItemTextColorInverted(const FLinearColor& InColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FTextBlockStyle ComboItemTextStyle;
	FFriendsComboStyle& SetComboItemTextStyle(const FTextBlockStyle& InTextStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FTextBlockStyle FriendsComboTextStyle;
	FFriendsComboStyle& SetFriendsComboTextStyle(const FTextBlockStyle& InTextStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FVector2D StatusButtonSize;
	FFriendsComboStyle& SetStatusButtonSize(const FVector2D& InStatusButtonSize);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FVector2D ActionComboButtonSize;
	FFriendsComboStyle& SetActionComboButtonSize(const FVector2D& InActionComboButtonSize);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FComboButtonStyle ActionComboButtonStyle;
	FFriendsComboStyle& SetActionComboButtonStyle(const FComboButtonStyle& ButtonStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FTextBlockStyle ActionComboButtonTextStyle;
	FFriendsComboStyle& SetActionComboButtonTextStyle(const FTextBlockStyle& TextStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin ComboMenuPadding;
	FFriendsComboStyle& SetComboMenuPadding(const FMargin& InPadding);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin ComboItemPadding;
	FFriendsComboStyle& SetComboItemPadding(const FMargin& InPadding);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin ComboItemContentPadding;
	FFriendsComboStyle& SetComboItemContentPadding(const FMargin& InPadding);

	/** Friends List Combo Button menu background image (left) */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush FriendComboBackgroundLeftBrush;
	FFriendsComboStyle& SetFriendComboBackgroundLeftBrush(const FSlateBrush& BrushStyle);

	/** Friends List Combo Button menu background image (right) */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush FriendComboBackgroundRightBrush;
	FFriendsComboStyle& SetFriendComboBackgroundRightBrush(const FSlateBrush& BrushStyle);

	/** Friends List Combo Button menu background image (left-flipped) - for MenuPlacement_ComboBoxRight menus */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush FriendComboBackgroundLeftFlippedBrush;
	FFriendsComboStyle& SetFriendComboBackgroundLeftFlippedBrush(const FSlateBrush& BrushStyle);

	/** Friends List Combo Button menu background image (right-flipped) - for MenuPlacement_ComboBoxRight menus */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush FriendComboBackgroundRightFlippedBrush;
	FFriendsComboStyle& SetFriendComboBackgroundRightFlippedBrush(const FSlateBrush& BrushStyle);

	/** Friends List Combo Button style */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FComboButtonStyle FriendListComboButtonStyle;
	FFriendsComboStyle& SetFriendsListComboButtonStyle(const FComboButtonStyle& ButtonStyle);
};

