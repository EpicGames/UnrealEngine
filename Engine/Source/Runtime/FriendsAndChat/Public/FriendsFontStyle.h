// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Fonts/SlateFontInfo.h"
#include "Math/Color.h"
#include "Styling/SlateWidgetStyle.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

#include "FriendsFontStyle.generated.h"

struct FSlateBrush;

/**
 * Interface for the services manager.
 */
USTRUCT()
struct FRIENDSANDCHAT_API FFriendsFontStyle
	: public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	// Default Constructor
	FFriendsFontStyle() { }

	// Default Destructor
	virtual ~FFriendsFontStyle() { }

	/**
	 * Override widget style function.
	 */
	virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override { }

	// Holds the widget type name
	static const FName TypeName;

	/**
	 * Get the type name.
	 * @return the type name
	 */
	virtual const FName GetTypeName() const override { return TypeName; };

	/**
	 * Get the default style.
	 * @return the default style
	 */
	static const FFriendsFontStyle& GetDefault();

	/** Font Style */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateFontInfo FriendsFontSmall;
	FFriendsFontStyle& SetFontSmall(const FSlateFontInfo& InFontStyle);

	/** Font Style */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateFontInfo FriendsFontSmallBold;
	FFriendsFontStyle& SetFontSmallBold(const FSlateFontInfo& InFontStyle);

	/** Font Style */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateFontInfo FriendsFontNormal;
	FFriendsFontStyle& SetFontNormal(const FSlateFontInfo& InFontStyle);

	/** Font Style */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateFontInfo FriendsFontNormalBold;
	FFriendsFontStyle& SetFontNormalBold(const FSlateFontInfo& InFontStyle);

	/** Font Style */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateFontInfo FriendsFontLarge;
	FFriendsFontStyle& SetFontLarge(const FSlateFontInfo& InFontStyle);

	/** Font Style */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateFontInfo FriendsFontLargeBold;
	FFriendsFontStyle& SetFontLargeBold(const FSlateFontInfo& InFontStyle);

	/** Font Style */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateFontInfo FriendsChatFont;
	FFriendsFontStyle& SetChatFont(const FSlateFontInfo& InFontStyle);

	/** Default Font Color */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FLinearColor DefaultFontColor;
	FFriendsFontStyle& SetDefaultFontColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category=Appearance)
	FLinearColor InvertedFontColor;
	FFriendsFontStyle& SetInvertedFontColor(const FLinearColor& InFontColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor DefaultDullFontColor;
	FFriendsFontStyle& SetDefaultDullFontColor(const FLinearColor& InFontColor);
};

