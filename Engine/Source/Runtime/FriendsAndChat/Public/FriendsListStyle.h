// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NameTypes.h"
#include "Math/Vector2D.h"
#include "UObject/ObjectMacros.h"
#include "Layout/Margin.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/SlateTypes.h"
#include "FriendsListStyle.generated.h"

/**
 * Interface for the services manager.
 */
USTRUCT()
struct FRIENDSANDCHAT_API FFriendsListStyle
	: public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	// Default Constructor
	FFriendsListStyle() 
	: BackButtonMargin(0.0, 0.0, 0.0, 1.0)
	, HeaderButtonMargin(0.0, 0.0, 1.0, 0.0)
	, FriendsListMargin(0.0, 1.0, 0.0, 0.0)
	, BackButtonContentMargin(15.0, 13.0, 0.0, 13.0)
	, FriendsListNoFriendsMargin(15.0, 10.0, 15.0, 0.0)
	, FriendsListHeaderMargin(45.0, 20.0, 0.0, 20.0)
	, FriendsListHeaderCountMargin(5.0, 20.0, 0.0, 20.0)
	, HeaderButtonContentMargin(0.0, 13.0)
	, FriendItemMargin(0.0, 17.0)
	, FriendItemStatusMargin(15.0, 0.0)
	, FriendItemPresenceMargin(27.0, 0.0, 35.0, 0.0)
	, FriendItemPlatformMargin(0, 0, 5, 0)
	, FriendItemTextScrollerMargin(5, 0, 0, 0)
	, ConfirmationBorderMargin(0.0, 0.0, 5.0, 0.0)
	, ConfirmationButtonMargin(5.0, 0.0, 0.0, 0.0)
	, ConfirmationButtonContentMargin(5.0, 20.0)
	, NoneFriendContentMargin(0)
	, NoneFriendContentHeight(75.f)
	, NoneFriendIconWidth(40.f)
	, SubMenuBackIconMargin(0.0, 0.0, 20.0, 0.0)
	, SubMenuPageIconMargin(20.0, 0.0)
	, RadioSettingTitleMargin(45.0, 10.0)
	, SubMenuSearchIconMargin(20.0, 50.0)
	, SubMenuSearchTextMargin(0.0, 0.0, 0.0, 0.0)
	, SubMenuBackButtonMargin(20.0)
	, SubMenuSettingButtonMargin(37.0, 20.0, 45.0, 20.0)
	, SubMenuListMargin(0.0, 5.0)
	, SubMenuSeperatorThickness(2.f)
	, PresenceSeperatorThickness(1.f)
	, FriendTipMargin(0, 10, 15, 10)
	, FriendTipSeperatorMargin(0, 10)
	, ToolTipMargin(10.0)
	, TipStatusMargin(0.0, 0.0, 15.0, 0.0)
	, AddButtonMargin(50, 12)
	, AddButtonSpacing(12.f, 25.f)
	{ }

	// Default Destructor
	virtual ~FFriendsListStyle() { }

	/**
	 * Override widget style function.
	 */
	virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override { }

	// Holds the widget type name
	static const FName TypeName;

	/**
	 * Get the type name.
	 * @return the type name
	 */
	virtual const FName GetTypeName() const override { return TypeName; };

	/**
	 * Get the default style.
	 * @return the default style
	 */
	static const FFriendsListStyle& GetDefault();

	// Friends List Style
	UPROPERTY( EditAnywhere, Category = Appearance )
	FButtonStyle GlobalChatButtonStyle;
	FFriendsListStyle& SetGlobalChatButtonStyle(const FButtonStyle& ButtonStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush GlobalChatIcon;
	FFriendsListStyle& SetGlobalChatIcon(const FSlateBrush& BrushStyle);

	/** Friends List Open Button style */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FButtonStyle FriendItemButtonStyle;
	FFriendsListStyle& SetFriendItemButtonStyle(const FButtonStyle& ButtonStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FButtonStyle ConfirmButtonStyle;
	FFriendsListStyle& SetConfirmButtonStyle(const FButtonStyle& ButtonStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FButtonStyle CancelButtonStyle;
	FFriendsListStyle& SetCancelButtonStyle(const FButtonStyle& ButtonStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateColor ButtonContentColor;
	FFriendsListStyle& SetButtonContentColor(const FSlateColor& InColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateColor ButtonHoverContentColor;
	FFriendsListStyle& SetButtonHoverContentColor(const FSlateColor& InColor);

	/** Optional content for the Add Friend button */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush ActionMenuArrowBrush;
	FFriendsListStyle& SetActionMenuArrowBrush(const FSlateBrush& BrushStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush ToolTipArrowBrush;
	FFriendsListStyle& SetToolTipArrowBrush(const FSlateBrush& BrushStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FButtonStyle BackButtonStyle;
	FFriendsListStyle& SetBackButtonStyle(const FButtonStyle& ButtonStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FButtonStyle HeaderButtonStyle;
	FFriendsListStyle& SetHeaderButtonStyle(const FButtonStyle& ButtonStyle);

	/** Friends List Action Button style */
	UPROPERTY( EditAnywhere, Category = Appearance )
	FButtonStyle FriendListActionButtonStyle;
	FFriendsListStyle& SetFriendListActionButtonStyle(const FButtonStyle& ButtonStyle);

	/** Optional content for the Add Friend button */
	UPROPERTY( EditAnywhere, Category = Appearance )
	FSlateBrush AddFriendButtonContentBrush;
	FFriendsListStyle& SetAddFriendButtonContentBrush(const FSlateBrush& BrushStyle);

	/** Friend Image brush style */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush StatusIconBrush;
	FFriendsListStyle& SetStatusIconBrush(const FSlateBrush& BrushStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush PCIconBrush;
	FFriendsListStyle& SetPCIconBrush(const FSlateBrush& BrushStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush ConsoleIconBrush;
	FFriendsListStyle& SetConsoleIconBrush(const FSlateBrush& BrushStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush EpicIconBrush;
	FFriendsListStyle& SetEpicIconBrush(const FSlateBrush& BrushStyle);

	/** Friend Image brush style */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush FriendImageBrush;
	FFriendsListStyle& SetFriendImageBrush(const FSlateBrush& BrushStyle);

	/** Offline brush style */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush OfflineBrush;
	FFriendsListStyle& SetOfflineBrush(const FSlateBrush& InOffLine);

	/** Online brush style */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush OnlineBrush;
	FFriendsListStyle& SetOnlineBrush(const FSlateBrush& InOnLine);

	/** Away brush style */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush AwayBrush;
	FFriendsListStyle& SetAwayBrush(const FSlateBrush& AwayBrush);

	/** Away brush style */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush SpectateBrush;
	FFriendsListStyle& SetSpectateBrush(const FSlateBrush& SpectateBrush);

	/** Friends window background */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush FriendsContainerBackground;
	FFriendsListStyle& SetFriendContainerBackground(const FSlateBrush& InFriendContainerBackground);

	/** Friends window background */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush FriendsListBackground;
	FFriendsListStyle& SetFriendsListBackground(const FSlateBrush& InBrush);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FEditableTextBoxStyle AddFriendEditableTextStyle;
	FFriendsListStyle& SetAddFriendEditableTextStyle(const FEditableTextBoxStyle& InEditableTextStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FVector2D UserPresenceImageSize;
	FFriendsListStyle& SetUserPresenceImageSize(const FVector2D& InUserPresenceImageSize);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush BackBrush;
	FFriendsListStyle& SetBackBrush(const FSlateBrush& Brush);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush SelectedOptionBrush;
	FFriendsListStyle& SetSelectedOptionBrush(const FSlateBrush& Brush);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush SettingsBrush;
	FFriendsListStyle& SetSettingsBrush(const FSlateBrush& Brush);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush SeperatorBrush;
	FFriendsListStyle& SetSeperatorBrush(const FSlateBrush& Brush);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush PresenceSeperatorBrush;
	FFriendsListStyle& SetPresenceSeperatorBrush(const FSlateBrush& Brush);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush FontSizeBrush;
	FFriendsListStyle& SetFontSizeBrush(const FSlateBrush& Brush);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush SearchBrush;
	FFriendsListStyle& SetSearchBrush(const FSlateBrush& Brush);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin BackButtonMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin HeaderButtonMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin FriendsListMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin BackButtonContentMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin FriendsListNoFriendsMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin FriendsListHeaderMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin FriendsListHeaderCountMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin HeaderButtonContentMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin FriendItemMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin FriendItemStatusMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin FriendItemPresenceMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin FriendItemPlatformMargin;
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin FriendItemTextScrollerMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin ConfirmationBorderMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin ConfirmationButtonMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin ConfirmationButtonContentMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin NoneFriendContentMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	float NoneFriendContentHeight;

	UPROPERTY(EditAnywhere, Category = Appearance)
	float NoneFriendIconWidth;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin SubMenuBackIconMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin SubMenuPageIconMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin RadioSettingTitleMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin SubMenuSearchIconMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin SubMenuSearchTextMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin SubMenuBackButtonMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin SubMenuSettingButtonMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin SubMenuListMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	float SubMenuSeperatorThickness;

	UPROPERTY(EditAnywhere, Category = Appearance)
	float PresenceSeperatorThickness;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin FriendTipMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin FriendTipSeperatorMargin;
		
	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin ToolTipMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin TipStatusMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin AddButtonMargin;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FVector2D AddButtonSpacing;


// Clan Settings

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush ClanDetailsBrush;
	FFriendsListStyle& SetClanDetailsBrush(const FSlateBrush& Brush);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush ClanMembersBrush;
	FFriendsListStyle& SetClanMembersBrush(const FSlateBrush& Brush);
};

