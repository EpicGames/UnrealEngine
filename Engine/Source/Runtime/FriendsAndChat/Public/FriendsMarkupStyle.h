// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "Layout/Margin.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/SlateTypes.h"
#include "FriendsMarkupStyle.generated.h"

/**
 * Interface for the services manager.
 */
USTRUCT()
struct FRIENDSANDCHAT_API FFriendsMarkupStyle
	: public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	// Default Constructor
	FFriendsMarkupStyle() 
	: SeperatorThickness(2.0)
	, MarkupPadding(0, 7)
	, ButtonPadding(30, 7)
	{ }

	// Default Destructor
	virtual ~FFriendsMarkupStyle() { }

	/**
	 * Override widget style function.
	 */
	virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override { }

	// Holds the widget type name
	static const FName TypeName;

	/**
	 * Get the type name.
	 * @return the type name
	 */
	virtual const FName GetTypeName() const override { return TypeName; };

	/**
	 * Get the default style.
	 * @return the default style
	 */
	static const FFriendsMarkupStyle& GetDefault();

	/** Markup Button style */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FButtonStyle MarkupButtonStyle;
	FFriendsMarkupStyle& SetMarkupButtonStyle(const FButtonStyle& ButtonStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FTextBlockStyle MarkupTextStyle;
	FFriendsMarkupStyle& SetMarkupTextStyle(const FTextBlockStyle& InTextStyle);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush MarkupBackground;
	FFriendsMarkupStyle& SetMarkupBackgroundBrush(const FSlateBrush& InChatBackgroundBrush);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateColor ButtonColor;
	FFriendsMarkupStyle& SetButtonColor(const FSlateColor& InColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateColor ButtonHoverColor;
	FFriendsMarkupStyle& SetButtonHoverColor(const FSlateColor& InColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateColor TipColor;
	FFriendsMarkupStyle& SetTipColor(const FSlateColor& InColor);

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush SeperatorBrush;
	FFriendsMarkupStyle& SetSeperatorBrush(const FSlateBrush& Brush);

	UPROPERTY(EditAnywhere, Category = Appearance)
	float SeperatorThickness;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin MarkupPadding;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin ButtonPadding;

};

