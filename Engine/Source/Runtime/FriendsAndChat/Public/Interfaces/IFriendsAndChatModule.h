// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

DECLARE_DELEGATE_RetVal(bool, FFriendsSystemReady )

/**
 * Interface for the Friends and chat manager.
 */
class IFriendsAndChatModule
	: public IModuleInterface
{
public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IFriendsAndChatModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IFriendsAndChatModule>("FriendsAndChat");
	}

	virtual void ShutdownStyle() = 0;

public:

	/** Virtual destructor. */
	virtual ~IFriendsAndChatModule() { }
};
